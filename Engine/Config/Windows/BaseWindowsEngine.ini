[Audio]
; Defining below allows switching to audio mixer using -audiomixer commandline
AudioCaptureModuleName=AudioCaptureWasapi
AudioMixerModuleName=AudioMixerWasapi
PlatformFormat=BINKA
PlatformStreamingFormat=BINKA

; Defines a platform-specific volume headroom (in dB) for audio to provide better platform consistency with respect to volume levels.
PlatformHeadroomDB=-3

[TextureStreaming]
; PoolSizeVRAMPercentage is how much percentage of GPU Dedicated VRAM should be used as a TexturePool cache for streaming textures (0 - unlimited streaming)
PoolSizeVRAMPercentage=70

[SystemSettings]
r.setres=1280x720
framepro.ScopeMinTimeMicroseconds=10
fx.NiagaraAllowRuntimeScalabilityChanges=1
; Perplatform to PerQualityLevel conversion mapping for platform groups
QualityLevelMapping="high"
; Enable single RHI thread stall for D3D11
r.Occlusion.SingleRHIThreadStall=1
r.Nanite.Streaming.ReservedResources=1
D3D12.Bindless.ResourceDescriptorHeapSize=32768
D3D12.Bindless.SamplerDescriptorHeapSize=2048
r.PSOPrecache.GlobalShaders=1
r.VRS.EnableSoftware=1
r.VRS.ContrastAdaptiveShading=1

[PlatformCrypto]
PlatformRequiresDataCrypto=True

[/Script/Engine.RendererSettings]
r.Shaders.RemoveUnusedInterpolators=1
r.Shadow.DetectVertexShaderLayerAtRuntime=1
; Substrate settings for more advanced shading on PC and editor by default.
r.Substrate.RoughDiffuse=1
r.Substrate.ShadingQuality=1
r.Substrate.TileCoord8bits=0
r.Substrate.Glints=1
r.Substrate.ClosuresPerPixel=4
r.Refraction.Blur=1

[OnlineSubsystem]
LocalPlatformName=WIN

[DeviceProfileManager]
DeviceProfileSelectionModule="WindowsDeviceProfileSelector"

[ShaderPlatformConfig PCD3D_SM5]
BindlessConfiguration=Disabled
SubstrateMaxClosuresPerPixel=1

[ShaderPlatformConfig PCD3D_SM6]
BindlessConfiguration=RayTracing

[ShaderPlatformConfig VULKAN_SM5]
BindlessConfiguration=Disabled
SubstrateMaxClosuresPerPixel=1

[ShaderPlatformConfig VULKAN_SM6]
; Bindless is necessary for Vulkan ray tracing
BindlessConfiguration=All

[/Script/WindowsRuntimeSettings.WindowsRuntimeSettings]
MaxNumTlsSlots=2048
MaxNumThreadsWithTlsSlots=512

[ConsoleVariables]
WindowsApplication.UseWorkerThreadForRawInput=False
r.PSOPrecache.KeepInMemoryForActiveMaterials=1
