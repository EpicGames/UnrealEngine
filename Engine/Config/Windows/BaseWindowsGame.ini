[Staging]
; maintain PC GDK compatibility with other GDK-based platforms
; titles should put Achivements.json in $(ProjectDir)/Config/Xbl instead
+AllowedDirectories=$(ProjectName)/Platforms/GDK/Config/OSS
