[InputPlatformSettings_Windows InputPlatformSettings]
MaxPlatformUserCount=8
input.DeviceMappingPolicy=1