[DataDrivenPlatformInfo]
bIsConfidential=false
IniParent=VulkanPC
PlatformNameAliases=Win64
TargetSettingsIniSectionName=/Script/WindowsTargetPlatform.WindowsTargetSettings
Freezing_b32Bit=false
Freezing_MaxFieldAlignment32=4
Freezing_MaxFieldAlignment64=8
Freezing_bAlignBases=true
bHasDedicatedGamepad=false
bDefaultInputStandardKeyboard=true
bInputSupportConfigurable=true
DefaultInputType=MouseAndKeyboard
bSupportsMouseAndKeyboard=true
bSupportsGamepad=true
bCanChangeGamepadType=true
bSupportsTouch=false
GlobalIdentifier=D1D5F296FF834A87B20FAAA9D6B8E9A6

NormalIconPath=Launcher/Windows/Platform_Windows_24x
LargeIconPath=Launcher/Windows/Platform_Windows_128x
XLargeIconPath=
AutoSDKPath=
TutorialPath=SharingAndReleasing
Windows:bIsEnabled=true
Mac:bIsEnabled=false
Linux:bIsEnabled=false
BinariesDirectoryName=Win64
bUsesHostCompiler=true
bUATClosesAfterLaunch=false
PlatformGroupName=Desktop
UBTPlatformName=Win64
ICUDataVersion=icudt64l


[ShaderPlatform PCD3D_SM5]
Language=D3D
MaxFeatureLevel=SM5
ShaderFormat=PCD3D_SM5
bIsPC=true
bSupportsDebugViewShaders=true
bSupportsRayTracing=false
bSupportsGPUScene=true
bSupportsNanite=false
bSupportsSceneDataCompressedTransforms=true
bSupportsComputeFramework=true
bSupportsLumenGI=true
bSupportsDistanceFields=true
bSupportsSSDIndirect=true
bRequiresVendorExtensionsForAtomics=true
bSupportsAnisotropicMaterials=true
bSupportsVariableRateShading=true
bSupportsGen5TemporalAA=true
bSupportsHairStrandGeometry=true
bSupportsFFTBloom=true
bSupportsDiaphragmDOF=true
; VPAndRTArrayIndexFromAnyShaderFeedingRasterizer is optional in our baseline DX11 version
bSupportsVertexShaderLayer=false
bSupportsInstancedStereo=true
bSupportsPercentageCloserShadows=true
bSupportsIndexBufferUAVs=true
bSupportsROV=true
bSupportsOIT=true
StaticShaderBindingLayoutSupport=Unsupported
SupportsMultiViewport=RuntimeDependent
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_D3D_SM5", "PC D3D SM5")
bSupportsDualSourceBlending=true
bSupportsNNEShaders=false
bSupportsIndependentSamplers=true
bSupportsVertexShaderUAVs=RuntimeGuaranteed

[PreviewPlatform PCD3D_SM5]
PlatformName=PC
ShaderFormat=PCD3D_SM5
ShaderPlatform=PCD3D_SM5
MenuTooltip=LOCTEXT("PreviewMenuTooltip_PCD3D_SM5", "DirectX 11, OpenGL 4.3+, PS4, XB1")

[ShaderPlatform PCD3D_SM6]
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_D3D_SM6", "PC D3D SM6")
Language=D3D
MaxFeatureLevel=SM6
ShaderFormat=PCD3D_SM6
bIsPC=true
MaxMeshShaderThreadGroupSize=128
bSupportsDebugViewShaders=true
bSupportsRayTracing=true
bSupportsRayTracingShaders=true
bSupportsInlineRayTracing=true
bInlineRayTracingRequiresBindless=true
bSupportsRayTracingCallableShaders=true
bSupportsRayTracingProceduralPrimitive=true
bSupportsRayTracingLinearSweptSpheres=true
bSupportsRayTracingIndirectInstanceData=true
bSupportsRayTracingClusterOps=true
bSupportsPathTracing=true
bSupportsShaderExecutionReordering=true
bSupportsGPUScene=true
bSupportsNanite=true
bSupportsSceneDataCompressedTransforms=true
bSupportsComputeFramework=true
bSupportsLumenGI=true
bSupportsDistanceFields=true
bSupportsSSDIndirect=true
bSupportsWaveOperations=RuntimeGuaranteed
MinimumWaveSize=4
MaximumWaveSize=128
bSupportsDxc=true
bSupportsDiaphragmDOF=true
bRequiresVendorExtensionsForAtomics=false
bSupportsUInt64ImageAtomics=true
bSupportsAnisotropicMaterials=true
bSupportsVariableRateShading=true
bSupportsGen5TemporalAA=true
bSupportsShaderRootConstants=true
bSupportsMeshShadersTier0=true
bSupportsMeshShadersTier1=true
bSupportsMeshShadersWithClipDistance=true
bSupportsHairStrandGeometry=true
bSupportsFFTBloom=true
bSupportsVertexShaderLayer=true
bSupportsInstancedStereo=true
bSupportsPercentageCloserShadows=true
bSupportsIndexBufferUAVs=true
bSupportsBindless=true
bSupportsDescriptorRange=true
StaticShaderBindingLayoutSupport=RayTracingOnly
bSupportsROV=true
bSupportsOIT=true
bSupportsRealTypes=RuntimeDependent
EnablesHLSL2021ByDefault=1
SupportsMultiViewport=RuntimeGuaranteed
bSupportsDualSourceBlending=true
bSupportsNNEShaders=true
bSupportsIndependentSamplers=true
bSupportsAdaptiveGBuffer=true
MaxSamplers=32
SupportsBarycentricsSemantic=RuntimeDependent
bSupportsWorkGraphs=true
bSupportsVertexShaderUAVs=RuntimeGuaranteed

[PreviewPlatform PCD3D_SM6]
PlatformName=PC
ShaderFormat=PCD3D_SM6
ShaderPlatform=PCD3D_SM6
MenuTooltip=LOCTEXT("PreviewMenuTooltip_PCD3D_SM6", "Windows using SM6 profile")

[ShaderPlatform PCD3D_ES3_1]
Language=D3D
MaxFeatureLevel=ES3_1
ShaderFormat=PCD3D_ES31
bIsPC=true
bIsMobile=true
bSupportsDistanceFields=true
bSupportsComputeFramework=true
bSupportsInstancedStereo=true
bSupportsVertexShaderLayer=true
bSupportsManualVertexFetch=false
bSupportsSceneDataCompressedTransforms=true
bSupportsDebugViewShaders=true
bSupportsShaderPipelines = false
bSupportsDualSourceBlending=true
bSupportsIndependentSamplers=true
bSupportsVariableRateShading=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_D3D_Mobile", "PC D3D Mobile")
SupportsMultiViewport=RuntimeDependent

[PreviewPlatform PCD3D_ES3_1]
PlatformName=PC
ShaderFormat=PCD3D_ES31
ShaderPlatform=PCD3D_ES3_1
MenuTooltip=LOCTEXT("PreviewMenuTooltip_PCD3D_Mobile", "Windows using Mobile profile.")
DeviceProfileName=Windows_Preview_ES31
PreviewDeviceProfileName=Windows_Preview_ES31

[PreviewShaderPlatform SP_PCD3D_ES3_1_NONSDF]
ParentShaderPlatform=PCD3D_ES3_1
bSupportsDistanceFields=false
bCanUsePreviewPlatformForMaterialValidation=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_D3D_Mobile_NonSDF", "PC D3D Mobile NonSDF")

[PreviewPlatform PCD3D_ES3_1_NONSDF]
PlatformName=PCD3D_ES3_1
ShaderFormat=PCD3D_ES31
ShaderPlatform=SP_PCD3D_ES3_1_NONSDF
PreviewShaderPlatform=SP_PCD3D_ES3_1_NONSDF
MenuTooltip=LOCTEXT("PreviewMenuTooltip_PCD3D_Mobile_NONSDF", "Windows using Mobile NonSDF profile.")
DeviceProfileName=Windows_Preview_ES31_NonSDF
PreviewDeviceProfileName=Windows_Preview_ES31_NonSDF

[ShaderPlatform OPENGL_PCES3_1]
Language=OpenGL
bIsPC=true
ShaderFormat=GLSL_150_ES31
MaxFeatureLevel=ES3_1
ShaderFormat=GLSL_150_ES31
bIsMobile=true
bSupportsManualVertexFetch=false
bSupportsClipDistance = false
bSupportsShaderPipelines = false
bSupportsVertexShaderSRVs = false
bSupportsUniformBufferObjects = true
bSupportsIndependentSamplers=false
bSupportsComputeFramework=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_PCGL_Mobile", "PC OpenGL Mobile")
