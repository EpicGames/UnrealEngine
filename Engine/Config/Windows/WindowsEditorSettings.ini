[/Script/AudioEditor.AudioEditorSettings]
+AudioMixerPlatforms=(DisplayName="XAudio2",ModuleName="AudioMixerXAudio2")
+AudioMixerPlatforms=(DisplayName="Wasapi",ModuleName="AudioMixerWasapi")
