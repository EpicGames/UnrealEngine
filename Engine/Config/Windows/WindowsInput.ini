[InputPlatformSettings_Windows InputPlatformSettings]
+HardwareDevices=(InputClassName="WindowsApplication",HardwareDeviceIdentifier="KBM",PrimaryDeviceType=KeyboardAndMouse,SupportedFeaturesMask=3)