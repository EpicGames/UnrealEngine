{
	"MainVersion": "10.0.22621.0",
	"MinVersion": "10.0.19041.0",
	"MaxVersion": "10.9.99999.0",

	"MinSoftwareVersion": "10.0.19041.0",

	"PreferredClangVersions": [
		"20.1.8-20.999",
		"20.1.0-20.1.7",
		"19.1.0-19.999",
		"18.1.8-18.999"
	],
	"MinimumClangVersion": "18.1.8",

	"//01.1": "this is a strange format, but instead of a version range, this maps min clang version for each MSVC version",
	"//01.2": "14.44 = VS2022 17.14.x -> clang 19",
	"//01.3": "14.42 = VS2022 17.12.x -> clang 18",
	"//01.4": "14.40 = VS2022 17.10.x -> clang 17",
	"//01.5": "14.37 = VS2022 17.7.x-17.9.x -> clang 16",
	"MinimumRequiredClangVersion": [
		"14.44-19",
		"14.42-18",
		"14.40-17",
		"14.37-16"
	],

	"//02.1": "Version number is the MSVC family, which is the version in the Visual Studio folder",
	"//02.2": "14.50.35717-14.50.99999 = VS2026 18.0",
	"//02.3": "14.44.35207-14.44.99999 = VS2022 17.14",
	"PreferredVisualCppVersions": [
		"14.50.35717-14.50.99999",
		"14.44.35207-14.44.99999"
	],

	"//03.1": "14.50.35717 -- VS2026 18.0.0 - 18.2.0: Internal compiler errors. Resolved with 14.50.35723",
	"//03.2": "14.44.35207 -- VS2022 17.14.0 - 17.4.3: Template compile error. Resolved with 14.44.35211",
	"//03.3": "14.40.x -- VS2022 17.10.0: Codegen issue causes runtime crash for AVX/2/512. 17.10.x: PGO optimize issue causes runtime crash",
	"//03.4": "14.39.x -- VS2022 17.9.0 - 17.9.3: Internal compiler errors. 17.9.x: Codegen issue causes runtime crash for AVX/2/512",
	"BannedVisualCppVersions": [
		"14.50.0-14.50.35722",
		"14.44.0-14.44.35210",
		"14.40.0-14.43.99999",
		"14.39.0-14.39.99999"
	],
	"MinimumVisualCppVersion": "14.38.33130",

	"PreferredIntelOneApiVersions": [
		"2026.0.0-2026.0.9999"
	],
	"MinimumIntelOneApiVersion": "2026.0.0",

	"VisualStudioSuggestedComponents": [
		"Microsoft.Net.Component.4.6.2.TargetingPack",
		"Microsoft.VisualStudio.Component.VC.Tools.x86.x64",
		"Microsoft.VisualStudio.Component.Windows11SDK.22621",
		"Microsoft.VisualStudio.Component.VC.Llvm.Clang",
		"Microsoft.VisualStudio.Workload.CoreEditor",
		"Microsoft.VisualStudio.Workload.ManagedDesktop",
		"Microsoft.VisualStudio.Workload.NativeDesktop",
		"Microsoft.VisualStudio.Workload.NativeGame",
		"Component.Unreal.Ide",
		"Component.Unreal.Debugger"
	],

	"VisualStudioSuggestedLinuxComponents": [
		"Microsoft.VisualStudio.Workload.NativeCrossPlat"
	],

	"//04.1": "Microsoft.VisualStudio.Component.VC.14.44.17.14.x86.x64 -- Only LTSC versions of MSVC should be suggested",
	"//04.2": "Microsoft.VisualStudio.Component.VC.14.44.17.14.ATL -- Match version of MSVC",
	"VisualStudio2022SuggestedComponents": [
		"Microsoft.VisualStudio.Component.VC.14.44.17.14.x86.x64",
		"Microsoft.VisualStudio.Component.VC.14.44.17.14.ATL"
	],

	"//05.1": "Microsoft.VisualStudio.Component.VC.14.50.18.0.x86.x64 -- Only LTSC versions of MSVC should be suggested",
	"//05.2": "Microsoft.VisualStudio.Component.VC.14.50.18.0.ATL -- Match version of MSVC",
	"VisualStudio2026SuggestedComponents": [
		"Microsoft.VisualStudio.Component.VC.14.50.18.0.x86.x64",
		"Microsoft.VisualStudio.Component.VC.14.50.18.0.ATL"
	],

	"MinimumVisualStudio2022Version": "17.8",
	"MinimumVisualStudio2026Version": "18.0",

	"//06.1": "X64 -- 0xA00 = Win10",
	"//06.2": "Arm64 -- 0xA00 = Win10",
	"MinimumWindowsX64TargetVersion": "0xA00",
	"MinimumWindowsArm64TargetVersion": "0xA00",

	"IncludeSDKFile": "$(EngineDir)/Config/Microsoft/GDK_SDK.json"
}
